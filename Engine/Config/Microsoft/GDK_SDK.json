{
	"MainGDKVersion": "260400",
	"MinGDKVersion": "250404",
	"MaxGDKVersion": "260499",

	"//GRDK_1": "240305 has a byte-ordering problem with PlayFab that causes compile errors",
	"BannedGDKVersions": [
		"240305"
	]

}
